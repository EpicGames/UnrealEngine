{
	"FileVersion": 3,
	"FriendlyName": "Razer Chroma Devices",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "Provides some functionality to set Razer Chroma effects at runtime.",
	"Category": "Peripherals",
	"EnabledByDefault": false,
	"IsBetaVersion": true,
	"CanContainContent": true,
	"RequiresBuildPlatform": false,
	"Modules": [
		{
			"Name": "RazerChromaDevices",
			"Type": "ClientOnlyNoCommandlet",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RazerChromaEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}