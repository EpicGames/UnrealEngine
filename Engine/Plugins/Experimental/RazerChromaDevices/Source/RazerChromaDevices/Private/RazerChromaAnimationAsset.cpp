// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaAnimationAsset.h"

#include "Misc/DataValidation.h"
#include "Misc/Paths.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RazerChromaAnimationAsset)

#define LOCTEXT_NAMESPACE "RazerChromaAnimationAsset"

#if WITH_EDITOR
bool URazerChromaAnimationAsset::ImportFromFile(const FString& InFileName, const uint8*& Buffer, const uint8* BufferEnd)
{
	// Strip the file name to just get the file name and extension from it
	AnimationName = FPaths::GetCleanFilename(InFileName);

	RawData.Reset();
	
	// Copy the data of the binary Razer Chroma anim file to a byte buffer on our UAsset.
	// We can use this byte buffer to play some files instead of reading in the .chroma
	// animation file itself at runtime, which is much safer. 
	const uint64 Size = static_cast<uint64>(BufferEnd - Buffer);
	RawData.AddUninitialized(static_cast<int32>(Size));
	FMemory::Memcpy(RawData.GetData(), Buffer, Size);
	
	return true;
}

EDataValidationResult URazerChromaAnimationAsset::IsDataValid(FDataValidationContext& Context) const
{
	EDataValidationResult Res = Super::IsDataValid(Context);

	if (AnimationName.IsEmpty())
	{
		Context.AddError(LOCTEXT("EmptyAnimationNameError", "A valid animation name is required!"));
		Res = CombineDataValidationResults(Res, EDataValidationResult::Invalid);
	}

	if (RawData.IsEmpty())
	{
		Context.AddError(LOCTEXT("EmptyByteBufferError", "There is no valid animation data!"));
		Res = CombineDataValidationResults(Res, EDataValidationResult::Invalid);
	}

	return Res;
}
#endif	// WITH_EDITOR

const FString& URazerChromaAnimationAsset::GetAnimationName() const
{
	return AnimationName;
}

const uint8* URazerChromaAnimationAsset::GetAnimByteBuffer() const
{
	return RawData.GetData();
}

#undef LOCTEXT_NAMESPACE
