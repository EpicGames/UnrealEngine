// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaDeviceProperties.h"
#include "RazerChromaAnimationAsset.h"
#include "Algo/Transform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RazerChromaDeviceProperties)

FRazerChromaPlayAnimationFile::FRazerChromaPlayAnimationFile()
	: FInputDeviceProperty(PropertyName())
{}

FName FRazerChromaPlayAnimationFile::PropertyName()
{
	static const FName PropName = TEXT("FRazerChromaPlayAnimationFile");
	return PropName;
}

void URazerChromaPlayAnimationFile::EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration)
{
	InternalProperty.AnimName = AnimAsset ? AnimAsset->GetAnimationName() : TEXT("");
	InternalProperty.AnimationByteBuffer = AnimAsset ? AnimAsset->GetAnimByteBuffer() : nullptr;
	InternalProperty.bLooping = bLooping;
}

FInputDeviceProperty* URazerChromaPlayAnimationFile::GetInternalDeviceProperty()
{
	return &InternalProperty;
}
