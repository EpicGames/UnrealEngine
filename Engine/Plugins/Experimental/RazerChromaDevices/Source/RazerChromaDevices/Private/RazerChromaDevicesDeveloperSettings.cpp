// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaDevicesDeveloperSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RazerChromaDevicesDeveloperSettings)


const bool URazerChromaDevicesDeveloperSettings::ShouldUseChromaAppInfoForInit() const
{
	return bUseChromaAppInfoForInit;
}

const FRazerChromaAppInfo& URazerChromaDevicesDeveloperSettings::GetRazerAppInfo() const
{
	return AppInfo;
}

const bool URazerChromaDevicesDeveloperSettings::IsRazerChromaEnabled() const
{
	return bIsRazerChromaEnabled;
}

const bool URazerChromaDevicesDeveloperSettings::ShouldCreateRazerInputDevice() const
{
	return bIsRazerChromaEnabled && bCreateRazerChromaInputDevice;
}

const URazerChromaAnimationAsset* URazerChromaDevicesDeveloperSettings::GetIdleAnimation() const
{
	return IdleAnimationAsset;
}

FName URazerChromaDevicesDeveloperSettings::GetCategoryName() const
{ 
	// This will make the developer settings show up in the "plugins" section under "project settings"
	static const FName SettingsCategory = TEXT("Plugins");

	return SettingsCategory;
}
