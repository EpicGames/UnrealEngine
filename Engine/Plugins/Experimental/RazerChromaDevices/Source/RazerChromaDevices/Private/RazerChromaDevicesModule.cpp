// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaDevicesModule.h"

#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "RazerChromaDeviceLogging.h"
#include "RazerChromaDevicesDeveloperSettings.h"
#include "RazerChromaInputDevice.h"
#include "RazerChromaDynamicAPI.h"
#include "RazerChromaSDKIncludes.h"
#include "RazerChromaAnimationAsset.h"
#include "RazerChromaFunctionLibrary.h"

namespace UE::RazerChroma
{
	/**
	 * The name of this modular feature plugin.
	 * This needs to be the same as the name used in the IMPLEMENT_MODULE macro. 
	 */
	static const FName FeatureName = TEXT("RazerChromaDevices");

	static const FString& GetRazerChromaDLLName()
	{
		static const FString RazerChromaEditorLibDLLName = TEXT("CChromaEditorLibrary64.dll");
	
		return RazerChromaEditorLibDLLName;
	}

	static void* GetChromaEditorDLL()
	{
		void* Res = nullptr;

#if RAZER_CHROMA_SUPPORT

		const FString PluginDirectory = IPluginManager::Get().FindPlugin(TEXT("RazerChromaDevices"))->GetBaseDir();
		const FString DllDirectory = FPaths::Combine(PluginDirectory, TEXT("Binaries/ThirdParty/Win64"));
		const FString DllPath = FPaths::Combine(DllDirectory, GetRazerChromaDLLName());

		if (FPaths::FileExists(DllPath))
		{
			FPlatformProcess::PushDllDirectory(*DllDirectory);
			Res = FPlatformProcess::GetDllHandle(*DllPath);
			FPlatformProcess::PopDllDirectory(*DllDirectory);
		}

		// TODO: Verify that this DLL file is signed by Razer and is not some injected DLL file

#endif	// #if RAZER_CHROMA_SUPPORT

		return Res;
	}

#if RAZER_CHROMA_SUPPORT
	static const RZRESULT InitChromaSDK()
	{
		const URazerChromaDevicesDeveloperSettings* Settings = GetDefault<URazerChromaDevicesDeveloperSettings>();
		
		// If you have settings about your application that you want to use to populate Razer Synapse, read them here.
		if (Settings->ShouldUseChromaAppInfoForInit())
		{
			if (!FRazerChromaEditorDynamicAPI::InitSDK)
			{
				return RZRESULT_INVALID;
			}

			const FRazerChromaAppInfo& SettingsAppInfo = Settings->GetRazerAppInfo();

			FRazerChromaAppInfo WhyMe = {};

			ChromaSDK::APPINFOTYPE AppInfo = {};			

			// Make sure that the application name will fit with our build config appends
			ensure(SettingsAppInfo.ApplicationTitle.Len() <= 236);

			// Outside of shipping builds, we will append the build config and target type to the application name
			// so that Razer Synapse recognizes them as different apps. This makes testing a little easier and ensures 
			// that the environment is clean for testing shipping builds.
	#if !UE_BUILD_SHIPPING

			TStringBuilder<256> TitleBuilder;

			TitleBuilder.Append(SettingsAppInfo.ApplicationTitle);
			TitleBuilder.Append(TEXT("_"));
			TitleBuilder.Append(LexToString(FApp::GetBuildConfiguration()));
			TitleBuilder.Append(TEXT("_"));
			TitleBuilder.Append(LexToString(FApp::GetBuildTargetType()));

			FCString::Strncpy(AppInfo.Title, TitleBuilder.ToString(), 256);

	#else

			FCString::Strncpy(AppInfo.Title, *SettingsAppInfo.ApplicationTitle, 256);

	#endif	// !UE_BUILD_SHIPPPING			


			FCString::Strncpy(AppInfo.Description, *SettingsAppInfo.ApplicationDescription, 1024);

			FCString::Strncpy(AppInfo.Author.Name, *SettingsAppInfo.AuthorName, 256);
			FCString::Strncpy(AppInfo.Author.Contact, *SettingsAppInfo.AuthorContact, 256);

			// Note: 63 (ERazerChromaDeviceTypes::All) is the highest number of options currently supported in v1.0.1.2
			const int32 MaxSupportedDevices = static_cast<int32>(ERazerChromaDeviceTypes::All);

			ensure(SettingsAppInfo.SupportedDeviceTypes <= MaxSupportedDevices && SettingsAppInfo.SupportedDeviceTypes >= 0);

			AppInfo.SupportedDevice = FMath::Clamp(SettingsAppInfo.SupportedDeviceTypes, 0, MaxSupportedDevices);

			AppInfo.Category = SettingsAppInfo.Category;			

			return FRazerChromaEditorDynamicAPI::InitSDK(&AppInfo);
		}

		// Otherwise, you don't want to specify any info about your application and will let Synapse auto-populate it

		if (!FRazerChromaEditorDynamicAPI::Init)
		{
			return RZRESULT_INVALID;
		}

		return FRazerChromaEditorDynamicAPI::Init();
	}

	static FAutoConsoleCommand GForceReInitCommand(
		TEXT("Razer.ForceReInit"),
		TEXT("Forcibly reinitalizes the Razer Chroma Editor API (calls Uninit, and then Init)."),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				if (FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get())
				{
					Module->ForceReinitalize();
				}
			})
	);

#endif //#if RAZER_CHROMA_SUPPORT
}

FRazerChromaDeviceModule* FRazerChromaDeviceModule::Get()
{
	return (FRazerChromaDeviceModule*)IModularFeatures::Get().GetModularFeatureImplementation(UE::RazerChroma::FeatureName, 0);
}

FName FRazerChromaDeviceModule::GetModularFeatureName()
{
	return UE::RazerChroma::FeatureName;
}

const FString FRazerChromaDeviceModule::RazerErrorToString(const int64 ErrorCode)
{
#if RAZER_CHROMA_SUPPORT
	const RZRESULT RazerError = static_cast<RZRESULT>(ErrorCode);

	static const TMap<RZRESULT, FString> ErrorMap =
		{
			{ RZRESULT_INVALID, TEXT("RZRESULT_INVALID")},
			{ RZRESULT_SUCCESS, TEXT("RZRESULT_SUCCESS")},
			{ RZRESULT_ACCESS_DENIED, TEXT("RZRESULT_ACCESS_DENIED")},
			{ RZRESULT_INVALID_HANDLE, TEXT("RZRESULT_INVALID_HANDLE")},
			{ RZRESULT_NOT_SUPPORTED, TEXT("RZRESULT_NOT_SUPPORTED")},
			{ RZRESULT_INVALID_PARAMETER, TEXT("RZRESULT_INVALID_PARAMETER")},
			{ RZRESULT_SERVICE_NOT_ACTIVE, TEXT("RZRESULT_SERVICE_NOT_ACTIVE")},
			{ RZRESULT_SINGLE_INSTANCE_APP, TEXT("RZRESULT_SINGLE_INSTANCE_APP")},
			{ RZRESULT_DEVICE_NOT_CONNECTED, TEXT("RZRESULT_DEVICE_NOT_CONNECTED")},
			{ RZRESULT_NOT_FOUND, TEXT("RZRESULT_NOT_FOUND")},
			{ RZRESULT_REQUEST_ABORTED, TEXT("RZRESULT_REQUEST_ABORTED")},
			{ RZRESULT_ALREADY_INITIALIZED, TEXT("RZRESULT_ALREADY_INITIALIZED")},
			{ RZRESULT_RESOURCE_DISABLED, TEXT("RZRESULT_RESOURCE_DISABLED")},
			{ RZRESULT_DEVICE_NOT_AVAILABLE, TEXT("RZRESULT_DEVICE_NOT_AVAILABLE")},
			{ RZRESULT_NOT_VALID_STATE, TEXT("RZRESULT_NOT_VALID_STATE")},
			{ RZRESULT_NO_MORE_ITEMS, TEXT("RZRESULT_NO_MORE_ITEMS")},
			{ RZRESULT_DLL_NOT_FOUND, TEXT("RZRESULT_DLL_NOT_FOUND")},
			{ RZRESULT_DLL_INVALID_SIGNATURE, TEXT("RZRESULT_DLL_INVALID_SIGNATURE")},
			{ static_cast<int32>(RZRESULT_FAILED), TEXT("RZRESULT_FAILED")}
		};

	if (const FString* Found = ErrorMap.Find(RazerError))
	{
		return *Found;	
	}
#endif	// #if RAZER_CHROMA_SUPPORT

	static const FString UnknownError = TEXT("Unknown Error");
	return UnknownError;
}

void FRazerChromaDeviceModule::StartupModule()
{
	IInputDeviceModule::StartupModule();

#if RAZER_CHROMA_SUPPORT

	UE_LOGF(LogRazerChroma, Log, "[%s] Razer Chroma module starting...", __func__);

	const URazerChromaDevicesDeveloperSettings* Settings = GetDefault<URazerChromaDevicesDeveloperSettings>();

	// Let us hotfix this stuff on or off in case it causes some issues
	if (!Settings->IsRazerChromaEnabled())
	{
		UE_LOGF(LogRazerChroma, Log, "[%s] URazerChromaDevicesDeveloperSettings::IsRazerChromaEnabled is false, Razer Chroma will not be available.", __func__);
		return;
	}
	
	RazerChromaEditorDLLHandle = UE::RazerChroma::GetChromaEditorDLL();
	if (RazerChromaEditorDLLHandle == nullptr)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load DLL '%ls'. Razer Chroma will not be available.", __func__, *UE::RazerChroma::GetRazerChromaDLLName());
		return;
	}
	
	bLoadedDynamicAPISuccessfully = FRazerChromaEditorDynamicAPI::LoadAPI(RazerChromaEditorDLLHandle);
	if (!bLoadedDynamicAPISuccessfully)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load the Razer Chroma Editor Dynamic Library! Razer Chroma will not be available.", __func__);
		return;
	}

	// Initialize the SDK	
	const RZRESULT Res = UE::RazerChroma::InitChromaSDK();
	
	// Ensure that we keep track of if we have loaded the API successfully or not for later.
	bLoadedDynamicAPISuccessfully &= (Res == RZRESULT_SUCCESS);

	// We only want to register the modular feature as being available if it has been successfully initialized
	if (bLoadedDynamicAPISuccessfully)
	{
		IModularFeatures::Get().RegisterModularFeature(UE::RazerChroma::FeatureName, this);
		UE_LOGF(LogRazerChroma, Log, "[%s] Razer Chroma module has successfully started!", __func__);

		// If there is a default animation set, then we can set it here
		if (const URazerChromaAnimationAsset* NewIdleAnimation = Settings->GetIdleAnimation())
		{
			URazerChromaFunctionLibrary::SetIdleAnimation(NewIdleAnimation);
			UE_LOGF(LogRazerChroma, Log, "[%s] Set default Idle Animation to %ls", __func__, *NewIdleAnimation->GetAnimationName());
		}
	}

	// This will be the result if you run on a machine which does not have the Razer Synapse client installed
	// (i.e. you don't have any razer products) We don't want to error here, as that would be expected. 
	if (Res == RZRESULT_DLL_NOT_FOUND)
	{
		UE_LOGF(LogRazerChroma, Log, "[%s] Failed to Init Razer Chroma Editor API. The Razer Synapse client is likely not installed on this machine. Error code %d (%ls)",
			__func__,
			Res,
			*FRazerChromaDeviceModule::RazerErrorToString(Res));
	}
	// If we failed for any other reason then it is not expected and we should log an error
	else if (Res != RZRESULT_SUCCESS)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to Init Razer Chroma Editor API. Error code %d (%ls)",
			__func__,
			Res,
			*FRazerChromaDeviceModule::RazerErrorToString(Res));
	}	

#else
	
	UE_LOGF(LogRazerChroma, Log, "[%s] RAZER_CHROMA_SUPPORT=0    No Razer Chroma Animation functionality will be available.", __func__);

#endif	// #if RAZER_CHROMA_SUPPORT
}

void FRazerChromaDeviceModule::ShutdownModule()
{
	IInputDeviceModule::ShutdownModule();

#if RAZER_CHROMA_SUPPORT

	// Modular feature is no longer available
	IModularFeatures::Get().UnregisterModularFeature(UE::RazerChroma::FeatureName, this);

	// Run some razer chroma specific cleanup
	CleanupSDK();

	// Free the DLL handle from the process
	if (RazerChromaEditorDLLHandle)
	{
		FPlatformProcess::FreeDllHandle(RazerChromaEditorDLLHandle);
	}

#endif	// #if RAZER_CHROMA_SUPPORT
}

TSharedPtr<IInputDevice> FRazerChromaDeviceModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
#if RAZER_CHROMA_SUPPORT
	// Only create the Razer input device if the DLL is available
	if (IsChromaAvailable() && GetDefault<URazerChromaDevicesDeveloperSettings>()->ShouldCreateRazerInputDevice())
	{
		TSharedPtr<FRazerChromaInputDevice> DevicePtr = MakeShared<FRazerChromaInputDevice>(InMessageHandler);
		return DevicePtr;
	}
	else
	{
		UE_LOGF(LogRazerChroma, 
			Log, 
			"[%s] URazerChromaDevicesDeveloperSettings::ShouldCreateRazerInputDevice is false, we will not create the Razer Chroma input device. Device Properties will not work.", 
			__func__);
	}
#endif // RAZER_CHROMA_SUPPORT

	return nullptr;
}

#if RAZER_CHROMA_SUPPORT

void FRazerChromaDeviceModule::CleanupSDK()
{
	// Disable idle animations
	if (FRazerChromaEditorDynamicAPI::SetUseIdleAnimations)
	{
		FRazerChromaEditorDynamicAPI::SetUseIdleAnimations(false);
	}

	// Stop playing all animations
	if (FRazerChromaEditorDynamicAPI::StopAllAnimations)
	{
		FRazerChromaEditorDynamicAPI::StopAllAnimations();
	}

	// Return any animations to disk
	if (FRazerChromaEditorDynamicAPI::CloseAll)
	{
		FRazerChromaEditorDynamicAPI::CloseAll();
	}

	// Finally, UnInit the whole sdk
	if (FRazerChromaEditorDynamicAPI::UnInit)
	{
		FRazerChromaEditorDynamicAPI::UnInit();
	}

	// Doing all of the above _should_ reset the state of Razer peripherals to the user's
	// default settings and make sure that the application is correctly removed from Razer Synapse...

	UE_LOGF(LogRazerChroma,
		Log,
		"[%s] Razer Chroma Editor library cleaned up.",
		__func__);
}

bool FRazerChromaDeviceModule::IsChromaAvailable() const
{
	return RazerChromaEditorDLLHandle != nullptr && bLoadedDynamicAPISuccessfully;
}

void FRazerChromaDeviceModule::ForceReinitalize()
{
	if (!RazerChromaEditorDLLHandle)
	{
		return;
	}

	// Force Uninit...
	CleanupSDK();

	// And re-init
	const RZRESULT Res = UE::RazerChroma::InitChromaSDK();
	UE_CLOGF(Res != RZRESULT_SUCCESS, LogRazerChroma, Error, "[%s] Failed to Init Razer Chroma Editor API. Error code %d", __func__, Res);

	// Ensure that we keep track of if we have loaded the API successfully or not for later.
	bLoadedDynamicAPISuccessfully &= (Res == RZRESULT_SUCCESS);
}

bool FRazerChromaDeviceModule::IsChromaRuntimeAvailable()
{
	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return false;
	}
	
	return Module->IsChromaAvailable();
}

const int32 FRazerChromaDeviceModule::FindOrLoadAnimationData(const URazerChromaAnimationAsset* AnimAsset)
{
	if (!AnimAsset)
	{
		return -1;
	}

	const uint8* AnimationByteBuffer = AnimAsset->GetAnimByteBuffer();

	if (!AnimationByteBuffer)
	{
		return -1;
	}

	const FString AnimName = AnimAsset->GetAnimationName();

	if (AnimName.IsEmpty())
	{
		return -1;
	}

	return FindOrLoadAnimationData(AnimName, AnimationByteBuffer);
}

const int32 FRazerChromaDeviceModule::FindOrLoadAnimationData(const FString& AnimName, const uint8* AnimByteBuffer)
{
	if (const int32* ExistingId = LoadedAnimationIdMap.Find(AnimName))
	{
		// If the animation was invalid last time we loaded it, try again.
		if (*ExistingId == -1)
		{
			const int32 ReloadedId = FRazerChromaEditorDynamicAPI::OpenAnimationFromMemory(AnimByteBuffer, TCHAR_TO_WCHAR(*AnimName));
			LoadedAnimationIdMap[AnimName] = ReloadedId;
			return ReloadedId;
		}

		// Otherwise it is valid, and we can just return it
		return *ExistingId;
	}

	// This is a new animation, lets load it from the Razer API
	const int32 LoadedAnimId = FRazerChromaEditorDynamicAPI::OpenAnimationFromMemory(AnimByteBuffer, TCHAR_TO_WCHAR(*AnimName));
	LoadedAnimationIdMap.Add(AnimName, LoadedAnimId);
	return LoadedAnimId;	
}

#endif	// #if RAZER_CHROMA_SUPPORT

IMPLEMENT_MODULE(FRazerChromaDeviceModule, RazerChromaDevices)
