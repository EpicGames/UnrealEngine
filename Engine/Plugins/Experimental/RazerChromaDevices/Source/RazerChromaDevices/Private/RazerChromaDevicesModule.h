// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDeviceModule.h"

#define UE_API RAZERCHROMADEVICES_API

class URazerChromaAnimationAsset;

/**
 * Input Device module that will create the Razer Chroma input device module.
 */
class FRazerChromaDeviceModule : public IInputDeviceModule
{
public:
	static UE_API FRazerChromaDeviceModule* Get();
	
	static UE_API FName GetModularFeatureName();

	/**
	* Returns a string representing the given Razer Error code
	* 
	* @see RzErrors.h
	*/
	static UE_API const FString RazerErrorToString(const int64 ErrorCode);

protected:
	
	//~ Begin IInputDeviceModule interface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	UE_API virtual TSharedPtr<IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;		
	//~ End IInputDeviceModule interface
	
#if RAZER_CHROMA_SUPPORT

	/**
     * Returns true if the Chroma DLL has been successfully loaded
     */
    UE_API bool IsChromaAvailable() const;

	/**
	* Cleans up the SDK and all currently playing animations.
	*/
	UE_API void CleanupSDK();

	/** Handle to the Razer Chroma dynamic DLL */
	void* RazerChromaEditorDLLHandle = nullptr;

	/** True if the dynamic API was successfully loaded from the DLL handle. */
	bool bLoadedDynamicAPISuccessfully = false;

	/**
	* A map of animation names (URazerChromaAnimationAsset::AnimationName) to their Animation ID
	* loaded in from Razer Chroma.
	*/
	TMap<FString, int32> LoadedAnimationIdMap;

public:

	/**
	* This will call the Unit and Init functions over again.
	* This can be useful if you need to completely reset the state of your razer devices
	* as if the application has been closed and re-opened again
	*/
	UE_API void ForceReinitalize();

	/**
	* Returns true if the Razer Chroma runtime is available (the DLL has been successfully loaded and all of the functions we request have been found)
	*/
	static UE_API bool IsChromaRuntimeAvailable();

	/**
	* Attempts to load the given animation property.
	*
	* Returns the int ID of the animation. -1 is invalid and means it failed to load.
	*/
	UE_API const int32 FindOrLoadAnimationData(const URazerChromaAnimationAsset* AnimAsset);

	/**
	* Attempts to load the given animation property.
	*
	* Returns the int ID of the animation. -1 is invalid and means it failed to load.
	*/
	UE_API const int32 FindOrLoadAnimationData(const FString& AnimName, const uint8* AnimByteBuffer);

#endif // #if RAZER_CHROMA_SUPPORT

};

#undef UE_API
