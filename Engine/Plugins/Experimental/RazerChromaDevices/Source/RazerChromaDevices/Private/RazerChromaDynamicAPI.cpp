// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaDynamicAPI.h"

#if RAZER_CHROMA_SUPPORT

#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h"		// GetProcAddress
#include "RazerChromaDeviceLogging.h"

FRazerChromaEditorDynamicAPI::INIT FRazerChromaEditorDynamicAPI::Init = nullptr;
FRazerChromaEditorDynamicAPI::INITSDK FRazerChromaEditorDynamicAPI::InitSDK = nullptr;
FRazerChromaEditorDynamicAPI::UNINIT FRazerChromaEditorDynamicAPI::UnInit = nullptr;
FRazerChromaEditorDynamicAPI::PLAYANIMATIONNAME FRazerChromaEditorDynamicAPI::PlayAnimationName = nullptr;
FRazerChromaEditorDynamicAPI::OPENANIMATIONFROMMEMORY FRazerChromaEditorDynamicAPI::OpenAnimationFromMemory = nullptr;
FRazerChromaEditorDynamicAPI::PLAYANIMATION FRazerChromaEditorDynamicAPI::PlayAnimationWithId = nullptr;
FRazerChromaEditorDynamicAPI::STOPANIMATION FRazerChromaEditorDynamicAPI::StopAnimation = nullptr;
FRazerChromaEditorDynamicAPI::CREATEEFFECT FRazerChromaEditorDynamicAPI::CreateEffect = nullptr;
FRazerChromaEditorDynamicAPI::CREATEKEYBOARDEFFECT FRazerChromaEditorDynamicAPI::CreateKeyboardEffect = nullptr;
FRazerChromaEditorDynamicAPI::CREATEHEADSETEFFECT FRazerChromaEditorDynamicAPI::CreateHeadsetEffect = nullptr;
FRazerChromaEditorDynamicAPI::CREATEMOUSEPADEFFECT FRazerChromaEditorDynamicAPI::CreateMousepadEffect = nullptr;
FRazerChromaEditorDynamicAPI::CREATEMOUSEEFFECT FRazerChromaEditorDynamicAPI::CreateMouseEffect = nullptr;
FRazerChromaEditorDynamicAPI::CREATEKEYPADEFFECT FRazerChromaEditorDynamicAPI::CreateKeypadEffect = nullptr;
FRazerChromaEditorDynamicAPI::CREATECHROMALINKEFFECT FRazerChromaEditorDynamicAPI::CreateChromaLinkEffect = nullptr;
FRazerChromaEditorDynamicAPI::SETEFFECT FRazerChromaEditorDynamicAPI::SetEffect = nullptr;
FRazerChromaEditorDynamicAPI::DELETEEFFECT FRazerChromaEditorDynamicAPI::DeleteEffect = nullptr;
FRazerChromaEditorDynamicAPI::SETIDLEANIMATION FRazerChromaEditorDynamicAPI::SetIdleAnimation = nullptr;
FRazerChromaEditorDynamicAPI::USEIDLEANIMATIONS FRazerChromaEditorDynamicAPI::SetUseIdleAnimations = nullptr;
FRazerChromaEditorDynamicAPI::STOPALLANIMATIONS FRazerChromaEditorDynamicAPI::StopAllAnimations = nullptr;
FRazerChromaEditorDynamicAPI::CLOSEALL FRazerChromaEditorDynamicAPI::CloseAll = nullptr;
FRazerChromaEditorDynamicAPI::PAUSEANIMATION FRazerChromaEditorDynamicAPI::PauseAnimation = nullptr;
FRazerChromaEditorDynamicAPI::RESUMEANIMATION FRazerChromaEditorDynamicAPI::ResumeAnimation = nullptr;
FRazerChromaEditorDynamicAPI::ISANIMATIONPLAYING FRazerChromaEditorDynamicAPI::IsAnimationPlaying = nullptr;
FRazerChromaEditorDynamicAPI::ISANIMATIONPAUSED FRazerChromaEditorDynamicAPI::IsAnimationPaused = nullptr;
FRazerChromaEditorDynamicAPI::GETTOTALDURATION FRazerChromaEditorDynamicAPI::GetTotalDuration = nullptr;
FRazerChromaEditorDynamicAPI::SETEVENTNAME FRazerChromaEditorDynamicAPI::SetEventName = nullptr;
FRazerChromaEditorDynamicAPI::USEFORWARDCHROMAEVENTS FRazerChromaEditorDynamicAPI::UseForwardChromaEvents = nullptr;

bool FRazerChromaEditorDynamicAPI::LoadAPI(void* RazerChromaEditorDLLHandle)
{
	if (!RazerChromaEditorDLLHandle)
	{
		UE_LOGF(LogRazerChroma, Fatal, "[%s] Razer Chroma Editor handle is null! This device never should have been created. Fatal exit!", __func__);
		return false;
	}
	
	// We have to disable this warning to allow for type casts to the function pointers we need
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	
	const HMODULE RazerModule = static_cast<HMODULE>(RazerChromaEditorDLLHandle);

	FRazerChromaEditorDynamicAPI::Init = reinterpret_cast<INIT>(GetProcAddress(RazerModule, "PluginInit"));
	FRazerChromaEditorDynamicAPI::InitSDK = reinterpret_cast<INITSDK>(GetProcAddress(RazerModule, "PluginInitSDK"));
	FRazerChromaEditorDynamicAPI::UnInit = reinterpret_cast<UNINIT>(GetProcAddress(RazerModule, "PluginUninit"));

	FRazerChromaEditorDynamicAPI::PlayAnimationName = reinterpret_cast<PLAYANIMATIONNAME>(GetProcAddress(RazerModule, "PluginPlayAnimationName"));
	FRazerChromaEditorDynamicAPI::OpenAnimationFromMemory = reinterpret_cast<OPENANIMATIONFROMMEMORY>(GetProcAddress(RazerModule, "PluginOpenAnimationFromMemory"));

	FRazerChromaEditorDynamicAPI::PlayAnimationWithId = reinterpret_cast<PLAYANIMATION>(GetProcAddress(RazerModule, "PluginPlayAnimationLoop"));

	FRazerChromaEditorDynamicAPI::StopAnimation = reinterpret_cast<STOPANIMATION>(GetProcAddress(RazerModule, "PluginStopAnimation"));
	FRazerChromaEditorDynamicAPI::CreateEffect = reinterpret_cast<CREATEEFFECT>(GetProcAddress(RazerModule, "PluginCreateEffect"));
	FRazerChromaEditorDynamicAPI::CreateKeyboardEffect = reinterpret_cast<CREATEKEYBOARDEFFECT>(GetProcAddress(RazerModule, "PluginCoreCreateKeyboardEffect"));
	FRazerChromaEditorDynamicAPI::CreateHeadsetEffect = reinterpret_cast<CREATEHEADSETEFFECT>(GetProcAddress(RazerModule, "PluginCoreCreateHeadsetEffect"));
	FRazerChromaEditorDynamicAPI::CreateMousepadEffect = reinterpret_cast<CREATEMOUSEPADEFFECT>(GetProcAddress(RazerModule, "PluginCoreCreateMousepadEffect"));
	FRazerChromaEditorDynamicAPI::CreateMouseEffect = reinterpret_cast<CREATEMOUSEEFFECT>(GetProcAddress(RazerModule, "PluginCoreCreateMouseEffect"));
	FRazerChromaEditorDynamicAPI::CreateKeypadEffect = reinterpret_cast<CREATEKEYPADEFFECT>(GetProcAddress(RazerModule, "PluginCoreCreateKeypadEffect"));
	FRazerChromaEditorDynamicAPI::CreateChromaLinkEffect = reinterpret_cast<CREATECHROMALINKEFFECT>(GetProcAddress(RazerModule, "PluginCoreCreateChromaLinkEffect"));
	FRazerChromaEditorDynamicAPI::SetEffect = reinterpret_cast<SETEFFECT>(GetProcAddress(RazerModule, "PluginCoreSetEffect"));
	FRazerChromaEditorDynamicAPI::DeleteEffect = reinterpret_cast<DELETEEFFECT>(GetProcAddress(RazerModule, "PluginCoreDeleteEffect"));
	FRazerChromaEditorDynamicAPI::SetIdleAnimation = reinterpret_cast<SETIDLEANIMATION>(GetProcAddress(RazerModule, "PluginSetIdleAnimation"));
	FRazerChromaEditorDynamicAPI::SetUseIdleAnimations = reinterpret_cast<USEIDLEANIMATIONS>(GetProcAddress(RazerModule, "PluginUseIdleAnimations"));
	FRazerChromaEditorDynamicAPI::StopAllAnimations = reinterpret_cast<STOPALLANIMATIONS>(GetProcAddress(RazerModule, "PluginStopAll"));	
	FRazerChromaEditorDynamicAPI::CloseAll = reinterpret_cast<CLOSEALL>(GetProcAddress(RazerModule, "PluginCloseAll"));
	FRazerChromaEditorDynamicAPI::PauseAnimation = reinterpret_cast<PAUSEANIMATION>(GetProcAddress(RazerModule, "PluginPauseAnimation"));
	FRazerChromaEditorDynamicAPI::ResumeAnimation = reinterpret_cast<RESUMEANIMATION>(GetProcAddress(RazerModule, "PluginResumeAnimation"));
	FRazerChromaEditorDynamicAPI::IsAnimationPlaying = reinterpret_cast<ISANIMATIONPLAYING>(GetProcAddress(RazerModule, "PluginIsPlaying"));
	FRazerChromaEditorDynamicAPI::IsAnimationPaused = reinterpret_cast<ISANIMATIONPAUSED>(GetProcAddress(RazerModule, "PluginIsAnimationPaused"));
	FRazerChromaEditorDynamicAPI::GetTotalDuration = reinterpret_cast<GETTOTALDURATION>(GetProcAddress(RazerModule, "PluginGetTotalDuration"));
	FRazerChromaEditorDynamicAPI::SetEventName = reinterpret_cast<SETEVENTNAME>(GetProcAddress(RazerModule, "PluginCoreSetEventName"));
	FRazerChromaEditorDynamicAPI::UseForwardChromaEvents = reinterpret_cast<USEFORWARDCHROMAEVENTS>(GetProcAddress(RazerModule, "PluginUseForwardChromaEvents"));

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

	return
		FRazerChromaEditorDynamicAPI::Init &&
		FRazerChromaEditorDynamicAPI::InitSDK &&
		FRazerChromaEditorDynamicAPI::UnInit &&
		FRazerChromaEditorDynamicAPI::PlayAnimationName &&
		FRazerChromaEditorDynamicAPI::OpenAnimationFromMemory &&
		FRazerChromaEditorDynamicAPI::PlayAnimationWithId &&
		FRazerChromaEditorDynamicAPI::StopAnimation &&
		FRazerChromaEditorDynamicAPI::CreateEffect &&
		FRazerChromaEditorDynamicAPI::CreateKeyboardEffect &&
		FRazerChromaEditorDynamicAPI::CreateHeadsetEffect &&
		FRazerChromaEditorDynamicAPI::CreateMousepadEffect &&
		FRazerChromaEditorDynamicAPI::CreateMouseEffect &&
		FRazerChromaEditorDynamicAPI::CreateKeypadEffect &&
		FRazerChromaEditorDynamicAPI::CreateChromaLinkEffect &&
		FRazerChromaEditorDynamicAPI::SetEffect &&
		FRazerChromaEditorDynamicAPI::DeleteEffect &&
		FRazerChromaEditorDynamicAPI::SetIdleAnimation && 
		FRazerChromaEditorDynamicAPI::SetUseIdleAnimations && 
		FRazerChromaEditorDynamicAPI::StopAllAnimations && 
		FRazerChromaEditorDynamicAPI::CloseAll &&
		FRazerChromaEditorDynamicAPI::PauseAnimation && 
		FRazerChromaEditorDynamicAPI::ResumeAnimation &&
		FRazerChromaEditorDynamicAPI::IsAnimationPlaying &&
		FRazerChromaEditorDynamicAPI::IsAnimationPaused &&
		FRazerChromaEditorDynamicAPI::GetTotalDuration &&
		FRazerChromaEditorDynamicAPI::SetEventName &&
		FRazerChromaEditorDynamicAPI::UseForwardChromaEvents;
}

#endif	// #if RAZER_CHROMA_SUPPORT