// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if RAZER_CHROMA_SUPPORT

#include "RazerChromaSDKIncludes.h"

/**
 * The Razer Chroma Editor Dynamic API, which should be loaded via the
 * CChromaEditorLibrary64.dll.
 *
 * To see a full list of exported functions from this DLL, you can use the
 * Microsoft Visual Studio command prompt and run the DUMPBIN command:
 * 
 *		dumpbin /EXPORTS CChromaEditorLibrary.dll
 * 
 * See https://assets.razerzone.com/dev_portal/C%2B%2B/en/index.html#c_interface for details
 */
struct FRazerChromaEditorDynamicAPI
{
	/**
	* Loads the Razer Chroma API, returns true if successful.
	* 
	* This will FATAL ERROR if RazerChromaDLLHandle is null.
	*/
	[[nodiscard]] static bool LoadAPI(void* RazerChromaEditorDLLHandle);
	
	// Declare some function pointer types that are the signature of the function we need
	typedef RZRESULT(*INIT)(void);

	// Initalize the Razer Chroma app with some specific settings
	typedef RZRESULT(*INITSDK)(ChromaSDK::APPINFOTYPE* AppInfo);

	typedef RZRESULT(*UNINIT)(void);
	
	typedef void (*PLAYANIMATIONNAME)(const wchar_t* path, bool loop);

	typedef int32 (*PLAYANIMATION)(int32 animationId, bool bLoop);

	typedef int32 (*OPENANIMATIONFROMMEMORY)(const BYTE* data, const wchar_t* name);

	typedef int32 (*STOPANIMATION)(int32 AnimationId);
	
	typedef RZRESULT(*CREATEEFFECT)(RZDEVICEID DeviceId, ChromaSDK::EFFECT_TYPE Effect, PRZPARAM pParam, RZEFFECTID *pEffectId);

	typedef RZRESULT(*CREATEKEYBOARDEFFECT)(ChromaSDK::Keyboard::EFFECT_TYPE Effect, PRZPARAM pParam, RZEFFECTID *pEffectId);

	typedef RZRESULT(*CREATEHEADSETEFFECT)(ChromaSDK::Headset::EFFECT_TYPE Effect, PRZPARAM pParam, RZEFFECTID *pEffectId);

	typedef RZRESULT(*CREATEMOUSEPADEFFECT)(ChromaSDK::Mousepad::EFFECT_TYPE Effect, PRZPARAM pParam, RZEFFECTID *pEffectId);

	typedef RZRESULT(*CREATEMOUSEEFFECT)(ChromaSDK::Mouse::EFFECT_TYPE Effect, PRZPARAM pParam, RZEFFECTID *pEffectId);

	typedef RZRESULT(*CREATEKEYPADEFFECT)(ChromaSDK::Keypad::EFFECT_TYPE Effect, PRZPARAM pParam, RZEFFECTID *pEffectId);

	typedef RZRESULT(*CREATECHROMALINKEFFECT)(ChromaSDK::ChromaLink::EFFECT_TYPE Effect, PRZPARAM pParam, RZEFFECTID *pEffectId);

	typedef RZRESULT(*SETEFFECT)(RZEFFECTID EffectId);

	typedef RZRESULT(*DELETEEFFECT)(RZEFFECTID EffectId);

	typedef void (*SETIDLEANIMATION)(int32 animationId);

	typedef void (*USEIDLEANIMATIONS)(bool bUseIdleAnimation);

	typedef void (*STOPALLANIMATIONS)();

	typedef void (*CLOSEALL)();

	typedef void (*PAUSEANIMATION)(int32 animationId);

	typedef void (*RESUMEANIMATION)(int32 animationId, bool loop);

	typedef bool (*ISANIMATIONPLAYING)(int32 animationId);

	typedef bool (*ISANIMATIONPAUSED)(int32 animationId);

	typedef float(*GETTOTALDURATION)(int32 animationId);

	typedef RZRESULT(*SETEVENTNAME)(LPCTSTR name);

	typedef void(*USEFORWARDCHROMAEVENTS)(bool toggle);

	// And some actual function pointers to those that we want to call...


	// Initialize the razer chroma editor library
	static INIT Init;

	// Initialize the razer chroma editor library with some additional description info about it
	static INITSDK InitSDK;

	// Uninitialize the razer chroma editor library
	static UNINIT UnInit;

	// Plays an animation via its file path
	static PLAYANIMATIONNAME PlayAnimationName;

	// Plays an animation via the animation ID
	static PLAYANIMATION PlayAnimationWithId;

	// Opens an animation from a byte buffer, returning the int32 animation id
	static OPENANIMATIONFROMMEMORY OpenAnimationFromMemory;

	static STOPANIMATION StopAnimation;
	static CREATEEFFECT CreateEffect;
	static CREATEKEYBOARDEFFECT CreateKeyboardEffect;
	static CREATEHEADSETEFFECT CreateHeadsetEffect;
	static CREATEMOUSEPADEFFECT CreateMousepadEffect;
	static CREATEMOUSEEFFECT CreateMouseEffect;
	static CREATEKEYPADEFFECT CreateKeypadEffect;
	static CREATECHROMALINKEFFECT CreateChromaLinkEffect;
	static SETEFFECT SetEffect;
	static DELETEEFFECT DeleteEffect;

	// Sets the idle animation of the application
	static SETIDLEANIMATION SetIdleAnimation;

	// Sets if we should use the current idle animation at all
	static USEIDLEANIMATIONS SetUseIdleAnimations;

	// Stops all animations that are currently playing
	static STOPALLANIMATIONS StopAllAnimations;

	// Closes all open animations so they can be reloaded from disk.
	// You should call this on application shutdown
	static CLOSEALL CloseAll;

	// Pauses the animation with the given int32 id
	static PAUSEANIMATION PauseAnimation;

	// Resume the animation with the given int32 id
	static RESUMEANIMATION ResumeAnimation;

	// Returns true if the given animation ID is currently playing.
	static ISANIMATIONPLAYING IsAnimationPlaying;

	// Returns true if the given anim ID is currently paused
	static ISANIMATIONPAUSED IsAnimationPaused;

	// Returns the duration in seconds with the given animation ID
	static GETTOTALDURATION GetTotalDuration;

	// Returns zero if Chroma event can be named
	static SETEVENTNAME SetEventName;

	// Sets if PlayAnimation should send event name to SetEventName
	static USEFORWARDCHROMAEVENTS UseForwardChromaEvents;
};

#endif	// #if RAZER_CHROMA_SUPPORT