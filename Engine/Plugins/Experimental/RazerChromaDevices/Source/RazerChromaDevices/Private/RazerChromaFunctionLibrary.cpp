// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaFunctionLibrary.h"

#include "RazerChromaAnimationAsset.h"
#include "RazerChromaDevicesModule.h"
#include "RazerChromaDynamicAPI.h"
#include "RazerChromaSDKIncludes.h"
#include "RazerChromaDeviceLogging.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RazerChromaFunctionLibrary)

namespace UE::Razer
{
	/**
	* Transforms Unreal Engine's FColor to Razer Chroma's packed uint32 representing RGB.
	*/
	static constexpr uint32 FColorToRazerRGB(const FColor& InColor)
	{
		return ((uint32)(((uint8)(InColor.R) | ((uint32)((uint8)(InColor.G)) << 8)) | (((uint32)(uint8)(InColor.B)) << 16)));
	};
};

bool URazerChromaFunctionLibrary::IsChromaRuntimeAvailable()
{
#if RAZER_CHROMA_SUPPORT
	return FRazerChromaDeviceModule::IsChromaRuntimeAvailable();
#else
	return false;
#endif	
}

bool URazerChromaFunctionLibrary::PlayChromaAnimation(const URazerChromaAnimationAsset* AnimToPlay, const bool bLooping)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return false;
	}

	if (!AnimToPlay)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid AnimToPlay!", __func__);
		return false;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return false;
	}
	
	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(AnimToPlay);
	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Chroma Animation %ls", __func__, *GetNameSafe(AnimToPlay));
		return false;
	}

	const int32 ActiveAnimId = FRazerChromaEditorDynamicAPI::PlayAnimationWithId(LoadedAnimId, bLooping);

	UE_CLOGF(ActiveAnimId == INDEX_NONE, LogRazerChroma, Error, "[%s] Failed to play animation!", __func__);
	UE_CLOGF(ActiveAnimId != INDEX_NONE, LogRazerChroma, Verbose, "[%s] Playing Razer Chroma Animation %ls (%d)", __func__, *GetNameSafe(AnimToPlay), ActiveAnimId);
	
	return (ActiveAnimId != INDEX_NONE);
#else
	return false;
#endif
}

bool URazerChromaFunctionLibrary::IsAnimationPlaying(const URazerChromaAnimationAsset* Anim)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return false;
	}

	if (!Anim)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid AnimToStop!", __func__);
		return false;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return false;
	}

	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(Anim);
	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Chroma Animation %ls", __func__, *GetNameSafe(Anim));
		return false;
	}

	return FRazerChromaEditorDynamicAPI::IsAnimationPlaying(LoadedAnimId);
#else 
	return false;
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::StopChromaAnimation(const URazerChromaAnimationAsset* AnimToStop)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return;
	}

	if (!AnimToStop)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid AnimToStop!", __func__);
		return;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return;
	}

	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(AnimToStop);
	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Chroma Animation %ls", __func__, *GetNameSafe(AnimToStop));
		return;
	}

	FRazerChromaEditorDynamicAPI::StopAnimation(LoadedAnimId);

	UE_LOGF(LogRazerChroma, Verbose, "[%s] Stopping Razer Chroma Animation %ls (%d)", __func__, *GetNameSafe(AnimToStop), LoadedAnimId);
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::PauseChromaAnimation(const URazerChromaAnimationAsset* AnimToPause)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return;
	}

	if (!AnimToPause)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid AnimToPause!", __func__);
		return;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return;
	}

	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(AnimToPause);
	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Chroma Animation %ls", __func__, *GetNameSafe(AnimToPause));
		return;
	}

	FRazerChromaEditorDynamicAPI::PauseAnimation(LoadedAnimId);
	UE_LOGF(LogRazerChroma, Verbose, "[%s] Pausing Razer Chroma Animation %ls (%d)", __func__, *GetNameSafe(AnimToPause), LoadedAnimId);
#endif	// #if RAZER_CHROMA_SUPPORT
}

bool URazerChromaFunctionLibrary::IsChromaAnimationPaused(const URazerChromaAnimationAsset* Anim)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return false;
	}

	if (!Anim)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid Anim!", __func__);
		return false;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return false;
	}

	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(Anim);
	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Chroma Animation %ls", __func__, *GetNameSafe(Anim));
		return false;
	}
	
	return FRazerChromaEditorDynamicAPI::IsAnimationPaused(LoadedAnimId);
#else
	return false;
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::ResumeChromaAnimation(const URazerChromaAnimationAsset* AnimToResume, const bool bLoop)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return;
	}

	if (!AnimToResume)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid AnimToResume!", __func__);
		return;
	}
	
	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return;
	}

	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(AnimToResume);
	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Chroma Animation %ls", __func__, *GetNameSafe(AnimToResume));
		return;
	}

	FRazerChromaEditorDynamicAPI::ResumeAnimation(LoadedAnimId, bLoop);
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::StopAllChromaAnimations()
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return;
	}

	FRazerChromaEditorDynamicAPI::StopAllAnimations();
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::SetIdleAnimation(const URazerChromaAnimationAsset* NewIdleAnimation)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return;
	}

	if (!NewIdleAnimation)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid NewIdleAnimation!", __func__);
		return;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return;
	}
	
	// Load the animation
	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(NewIdleAnimation);

	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Idle Animation!", __func__);
		return;
	}

	FRazerChromaEditorDynamicAPI::SetIdleAnimation(LoadedAnimId);
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::SetUseIdleAnimation(const bool bUseIdleAnimation)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return;
	}

	FRazerChromaEditorDynamicAPI::SetUseIdleAnimations(bUseIdleAnimation);
#endif	// #if RAZER_CHROMA_SUPPORT
}

float URazerChromaFunctionLibrary::GetTotalDuration(const URazerChromaAnimationAsset* Anim)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return 0;
	}

	if (!Anim)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Invalid Anim!", __func__);
		return 0;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get();
	if (!Module)
	{
		return 0;
	}

	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(Anim);
	if (LoadedAnimId == INDEX_NONE)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] Failed to load Chroma Animation %ls", __func__, *GetNameSafe(Anim));
		return 0;
	}

	return FRazerChromaEditorDynamicAPI::GetTotalDuration(LoadedAnimId);
#else
	return 0;
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::SetAllDevicesStaticColor(const FColor& ColorToSet, const ERazerChromaDeviceTypes DeviceTypes /* = ERazerChromaDeviceTypes::All */)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return;
	}

	// TODO: A wrapper for the color types (static, breathing, cycling, etc) would be nice
	// It is different for each device though, so we probably would want some explicit function wrappers for each one

	const uint32 CurrentColor = UE::Razer::FColorToRazerRGB(ColorToSet);
	
	if (EnumHasAnyFlags(DeviceTypes, ERazerChromaDeviceTypes::Keyboards))
	{
		RZEFFECTID KBId = {};
		ChromaSDK::Keyboard::STATIC_EFFECT_TYPE KBDef = {};
		KBDef.Color = CurrentColor;
		FRazerChromaEditorDynamicAPI::CreateKeyboardEffect(ChromaSDK::Keyboard::CHROMA_STATIC, &KBDef, &KBId);
		FRazerChromaEditorDynamicAPI::SetEffect(KBId);	
	}

	if (EnumHasAnyFlags(DeviceTypes, ERazerChromaDeviceTypes::Mice))
	{
		RZEFFECTID MouseId = {};
		ChromaSDK::Mouse::STATIC_EFFECT_TYPE MouseDef = {};
		MouseDef.LEDId = ChromaSDK::Mouse::RZLED_ALL;
		MouseDef.Color = CurrentColor;
		FRazerChromaEditorDynamicAPI::CreateMouseEffect(ChromaSDK::Mouse::CHROMA_STATIC, &MouseDef, &MouseId);
		FRazerChromaEditorDynamicAPI::SetEffect(MouseId);	
	}	

	if (EnumHasAnyFlags(DeviceTypes, ERazerChromaDeviceTypes::Mousepads))
	{
		RZEFFECTID MousepadId = {};
		ChromaSDK::Mousepad::STATIC_EFFECT_TYPE MousepadDef = {};
		MousepadDef.Color = CurrentColor;
		FRazerChromaEditorDynamicAPI::CreateMousepadEffect(ChromaSDK::Mousepad::CHROMA_STATIC, &MousepadDef, &MousepadId);
		FRazerChromaEditorDynamicAPI::SetEffect(MousepadId);	
	}

	if (EnumHasAnyFlags(DeviceTypes, ERazerChromaDeviceTypes::Headset))
	{
		RZEFFECTID HeadsetId = {};
		ChromaSDK::Headset::STATIC_EFFECT_TYPE HeadsetDef = {};
		HeadsetDef.Color = CurrentColor;
		FRazerChromaEditorDynamicAPI::CreateHeadsetEffect(ChromaSDK::Headset::CHROMA_STATIC, &HeadsetDef, &HeadsetId);
		FRazerChromaEditorDynamicAPI::SetEffect(HeadsetId);	
	}

	if (EnumHasAnyFlags(DeviceTypes, ERazerChromaDeviceTypes::ChromaLink))
	{
		RZEFFECTID ChromaLinkId = {};
		ChromaSDK::ChromaLink::STATIC_EFFECT_TYPE ChromaLinkDef = {};
		ChromaLinkDef.Color = CurrentColor;
		FRazerChromaEditorDynamicAPI::CreateChromaLinkEffect(ChromaSDK::ChromaLink::CHROMA_STATIC, &ChromaLinkDef, &ChromaLinkId);
		FRazerChromaEditorDynamicAPI::SetEffect(ChromaLinkId);	
	}

	UE_LOGF(LogRazerChroma, Verbose, "[%s] Set static light color to %ls on device types %ls",
		__func__,
		*ColorToSet.ToString(),
		*LexToString(DeviceTypes));
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::SetAllDevicesStaticColor(const FColor& ColorToSet, const int32 DeviceTypes)
{
	URazerChromaFunctionLibrary::SetAllDevicesStaticColor(ColorToSet, static_cast<ERazerChromaDeviceTypes>(DeviceTypes));
}

FString URazerChromaFunctionLibrary::LexToString(const ERazerChromaDeviceTypes DeviceTypes)
{
	const ERazerChromaDeviceTypes EnumVal = static_cast<ERazerChromaDeviceTypes>(DeviceTypes);
	
	if (EnumVal == ERazerChromaDeviceTypes::None)
	{
		return TEXT("None");
	}
	else if (EnumVal == ERazerChromaDeviceTypes::All)
	{
		return TEXT("All");
	}
	
	FString Result = TEXT("");

#define DEVICE_TYPE(StatusFlag, DisplayName) if( EnumHasAnyFlags(EnumVal, StatusFlag) ) Result += (FString(DisplayName) + TEXT("|"));
	DEVICE_TYPE(ERazerChromaDeviceTypes::Keyboards, TEXT("Keyboards"));
	DEVICE_TYPE(ERazerChromaDeviceTypes::Mice, TEXT("Mice"));
	DEVICE_TYPE(ERazerChromaDeviceTypes::Headset, TEXT("Headset"));
	DEVICE_TYPE(ERazerChromaDeviceTypes::Mousepads, TEXT("Mousepads"));
	DEVICE_TYPE(ERazerChromaDeviceTypes::Keypads, TEXT("Keypads"));
	DEVICE_TYPE(ERazerChromaDeviceTypes::ChromaLink, TEXT("ChromaLink"));
#undef DEVICE_TYPE

	Result.RemoveFromEnd(TEXT("|"));
	return Result;
}

FString URazerChromaFunctionLibrary::Conv_RazerChromaDeviceTypesToString(const int32 DeviceTypes)
{
	return URazerChromaFunctionLibrary::LexToString(static_cast<ERazerChromaDeviceTypes>(DeviceTypes));
}

int32 URazerChromaFunctionLibrary::SetEventName(const FString& name)
{
#if RAZER_CHROMA_SUPPORT
	if (!URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		return -1;
	}

	return FRazerChromaEditorDynamicAPI::SetEventName(TCHAR_TO_WCHAR(*name));
#else
	return -1;
#endif	// #if RAZER_CHROMA_SUPPORT
}

void URazerChromaFunctionLibrary::UseForwardChromaEvents(const bool bToggle)
{
#if RAZER_CHROMA_SUPPORT
	if (URazerChromaFunctionLibrary::IsChromaRuntimeAvailable())
	{
		FRazerChromaEditorDynamicAPI::UseForwardChromaEvents(bToggle);
	}
#endif	// #if RAZER_CHROMA_SUPPORT
}
