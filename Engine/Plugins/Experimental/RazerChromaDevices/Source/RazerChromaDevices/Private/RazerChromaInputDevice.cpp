// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaInputDevice.h"

#if RAZER_CHROMA_SUPPORT

#include "RazerChromaDeviceLogging.h"
#include "RazerChromaDeviceProperties.h"
#include "RazerChromaDevicesModule.h"
#include "RazerChromaDynamicAPI.h"
#include "RazerChromaFunctionLibrary.h"
#include "RazerChromaSDKIncludes.h"
#include "RazerChromaDevicesDeveloperSettings.h"
#include "RazerChromaAnimationAsset.h"

FRazerChromaInputDevice::FRazerChromaInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	: MessageHandler(InMessageHandler)
{	

}

FRazerChromaInputDevice::~FRazerChromaInputDevice()
{
	// TODO: Delete any active effects
}

void FRazerChromaInputDevice::Tick(float DeltaTime)
{
	// Required by IInputDevice
}

void FRazerChromaInputDevice::SendControllerEvents()
{
	// Required by IInputDevice
}

void FRazerChromaInputDevice::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

bool FRazerChromaInputDevice::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	// TODO: Any debug commands could go here
	
	// required by IInputDevice interface
	return false;
}

void FRazerChromaInputDevice::SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
{
	// required by IInputDevice interface
}

void FRazerChromaInputDevice::SetChannelValues(int32 ControllerId, const FForceFeedbackValues& values)
{
	// required by IInputDevice interface
}

bool FRazerChromaInputDevice::SupportsForceFeedback(int32 ControllerId)
{
	// required by IInputDevice interface
	return false;
}

void FRazerChromaInputDevice::SetLightColor(int32 ControllerId, FColor Color)
{
	// TODO: Do we want to support this, or just use SetDeviceProperty?
}

void FRazerChromaInputDevice::ResetLightColor(int32 ControllerId)
{
	// TODO: Do we want to support this, or just use SetDeviceProperty?
}

void FRazerChromaInputDevice::SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property)
{
	if (!Property)
	{
		return;
	}
	
	if (Property->Name == FInputDeviceLightColorProperty::PropertyName())
	{
		HandlePropertySetLightColor(*(static_cast<const FInputDeviceLightColorProperty*>(Property)));
	}
	else if (Property->Name == FRazerChromaPlayAnimationFile::PropertyName())
	{
		HandlePlayAnimationFile(*(static_cast<const FRazerChromaPlayAnimationFile*>(Property)));
	}
}

void FRazerChromaInputDevice::HandlePropertySetLightColor(const FInputDeviceLightColorProperty& LightProperty)
{
	URazerChromaFunctionLibrary::SetAllDevicesStaticColor(LightProperty.Color);
}

void FRazerChromaInputDevice::HandlePlayAnimationFile(const FRazerChromaPlayAnimationFile& Property)
{
	if (Property.AnimName.IsEmpty())
	{
		UE_LOGF(LogRazerChroma, Error, "[%s]Invalid animation name!", __func__);
		return;
	}

	if (!Property.AnimationByteBuffer)
	{
		UE_LOGF(LogRazerChroma, Error, "[%s] There is no animation data for chroma effect %ls", __func__, *Property.AnimName);
		return;
	}

	FRazerChromaDeviceModule* Module = FRazerChromaDeviceModule::Get(); 
	if (!Module)
	{
		return;
	}
	
	const int32 LoadedAnimId = Module->FindOrLoadAnimationData(Property.AnimName, Property.AnimationByteBuffer);

	if (LoadedAnimId == -1)
	{
		UE_LOGF(LogRazerChroma, Warning, "[%s] Failed to load animation from memory", __func__);
		return;
	}
	
	const int32 ActiveAnimId = FRazerChromaEditorDynamicAPI::PlayAnimationWithId(LoadedAnimId, Property.bLooping);
	UE_CLOGF(ActiveAnimId == -1, LogRazerChroma, Error, "[%s] Failed to play animation!", __func__);
}

#endif	// #if RAZER_CHROMA_SUPPORT