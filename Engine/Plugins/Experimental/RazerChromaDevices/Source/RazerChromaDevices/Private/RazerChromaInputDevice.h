// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if RAZER_CHROMA_SUPPORT

#include "IInputDevice.h"
#include "GenericPlatform/IInputInterface.h"
#include "RazerChromaSDKIncludes.h"
#include "RazerChromaDeviceProperties.h"

/**
 * This input device will handle the setting of device properties on Razer Chroma.
 * It will not actually send any input events to the message handler, just set device
 * properties like lights and other effects.
 *
 * TODO: Can we refactor SetDeviceProperty out into some other interface function so that we don't need all the other virtual functions?
 */
class FRazerChromaInputDevice : public IInputDevice
{
public:
	explicit FRazerChromaInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);
	virtual ~FRazerChromaInputDevice() override;
	
protected:

	//~ Begin IInputDevice interface
	virtual void Tick(float DeltaTime) override;
	virtual void SendControllerEvents() override;
	virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
	virtual void SetChannelValue (int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override;
	virtual void SetChannelValues (int32 ControllerId, const FForceFeedbackValues &values) override;
	virtual bool SupportsForceFeedback(int32 ControllerId) override;	
	virtual void SetLightColor(int32 ControllerId, FColor Color) override;
	virtual void ResetLightColor(int32 ControllerId) override;
	virtual void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) override;
	//~End IInputDevice interface
	
	/** Message handler that we can use to tell the engine about input events */
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;

	/**
     * Pointer to the Razer Chroma DLL handle that has been loaded by the module.
     */
    void* RazerChromaDLLHandle;
	
	void HandlePropertySetLightColor(const FInputDeviceLightColorProperty& LightProperty);

	void HandlePlayAnimationFile(const FRazerChromaPlayAnimationFile& Property);
};

#endif	// #if RAZER_CHROMA_SUPPORT