// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "RazerChromaAnimationAsset.generated.h"

#define UE_API RAZERCHROMADEVICES_API

class FDataValidationContext;

/**
 * Represents a single ".chroma" Razer animation file that can be played.
 */
UCLASS(MinimalAPI, BlueprintType, hidecategories = (Object), Meta = (DisplayName = "Razer Chroma Asset"))
class URazerChromaAnimationAsset : public UObject
{
	GENERATED_BODY()
public:

#if WITH_EDITOR
	/**
	 * Imports the data for this razer chroma asset from the given binary file buffer.
	 *
	 * @param InFileName	The name of the file that this is being imported from.
	 * @param Buffer		Pointer to the byte buffer of chroma animation data
	 * @param BufferEnd		Pointer to the end of the animation byte buffer
	 *
	 * @return				True if successfully imported, false otherwise.
	 */
	UE_API bool ImportFromFile(const FString& InFileName, const uint8*& Buffer, const uint8* BufferEnd);
#endif	// WITH_EDITOR

	/**
	* The name of this animation that Razer Chroma should consider.
	*
	* This is automatically set based on the .chroma animation file when imported, but you can rename it if you desire.
	* 
	* @see URazerChromaAnimationAsset::AnimationName
	*/
	UE_API const FString& GetAnimationName() const;

	/**
	* The raw byte data imported from the asset factory when reading a .chroma file.
	* 
	* @see URazerChromaAnimationAsset::RawData
	*/
	UE_API const uint8* GetAnimByteBuffer() const;
	
protected:

#if WITH_EDITOR
	UE_API virtual EDataValidationResult IsDataValid(FDataValidationContext& Context) const override;
#endif

	// TODO: This Animation Name should be unique asset types. We should make an editor name validator for it

	/**
	* The name of this animation that Razer Chroma should consider.
	* 
	* This is automatically set based on the .chroma animation file when imported, but you can rename it if you desire.
	*/
	UPROPERTY(EditAnywhere, NoClear, Category="Razer")
	FString AnimationName;
	
	/** 
	* The raw byte data imported from the asset factory when reading a .chroma file.
	*/
	UPROPERTY()
	TArray<uint8> RawData;
};

#undef UE_API
