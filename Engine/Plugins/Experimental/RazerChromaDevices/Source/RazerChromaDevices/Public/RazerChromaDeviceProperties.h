// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/IInputInterface.h"
#include "GameFramework/InputDeviceProperties.h"

#include "RazerChromaDeviceProperties.generated.h"

class URazerChromaAnimationAsset;

/**
 * Plays a Razer Chroma animation file (*.chroma files)
 */
struct FRazerChromaPlayAnimationFile : public FInputDeviceProperty
{
	FRazerChromaPlayAnimationFile();

	static FName PropertyName();

	/** The name of this animation that Razer Chroma should load */
	FString AnimName;

	const uint8* AnimationByteBuffer = nullptr;

	/** 
	* If true, then this animation should loop when played
	*/
	bool bLooping = false;	
};

UCLASS(Blueprintable, BlueprintType, meta = (DisplayName = "Razer Chroma Play Animation File"))
class URazerChromaPlayAnimationFile : public UInputDeviceProperty
{
	GENERATED_BODY()
	
protected:

	virtual void EvaluateDeviceProperty_Implementation(const FPlatformUserId PlatformUser, const FInputDeviceId DeviceId, const float DeltaTime, const float Duration) override;
	virtual FInputDeviceProperty* GetInternalDeviceProperty() override;
	
	/** The chroma animation to play */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Razer")
	TObjectPtr<URazerChromaAnimationAsset> AnimAsset;

	/**
	* If true, then this animation should loop when played
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Razer")
	bool bLooping = false;

	FRazerChromaPlayAnimationFile InternalProperty = {};
};