// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "RazerChromaDevicesDeveloperSettings.generated.h"

#define UE_API RAZERCHROMADEVICES_API

class URazerChromaAnimationAsset;

/**
* Bitmask options for Razer Chroma device types.
*/
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ERazerChromaDeviceTypes : int32
{
	None = 0x0000 UMETA(Hidden),
	Keyboards = 0x0001,
	Mice = 0x0002,
	Headset = 0x0004,
	Mousepads = 0x0008,
	Keypads = 0x0010,
	ChromaLink = 0x0020,
	All = (Keyboards | Mice | Headset | Mousepads | Keypads | ChromaLink) UMETA(Hidden)
};
ENUM_CLASS_FLAGS(ERazerChromaDeviceTypes);

/**
* This information will be used to populate data in Razer Synapse.
*/
USTRUCT()
struct FRazerChromaAppInfo
{
	GENERATED_BODY()

	/**
	* The name of your application to report to Razer Synapse.
	* 
	* In non-shipping builds, this will have the build target and type appended to it, which will make it show up as :
	* 
	*	"ApplicationTitle_<BuildType>_<BuildTarget>" in Razer Synapse. 
	* 
	* This is expected, and in a shipping build it will be displayed as only: 
	* 
	*	"ApplicationTitle"
	* 
	* This is necessary because if you register multiple executable names (i.e. <YourGame>.exe, <YourGame>-Win64-Shipping.exe, and <YourGame>-Win64-Test.exe)
	* then Razer Synapse will only use the _first_ executable that you ran with this configuration. Appending the target names makes them unique in Synapse and easier
	* to test.
	* 
	* Cannot be empty.
	* 
	* Char limit of 236.
	*/
	UPROPERTY(EditAnywhere, Category="Razer Chroma App Config", NoClear, meta=(MaxLength=236))
	FString ApplicationTitle = TEXT("Your Game Name here");

	/**
	* The description of your application to report to Razer Synapse.
	*
	* Cannot be empty.
	* 
	* Char limit of 1024. 
	*/
	UPROPERTY(EditAnywhere, Category = "Razer Chroma App Config", NoClear, meta = (MaxLength = 1024))
	FString ApplicationDescription = TEXT("Describe your game here");
	
	/**
	* Name of the author of this application (company name) 
	* 
	* Cannot be empty.
	* 
	* Char limit of 256.
	*/
	UPROPERTY(EditAnywhere, Category = "Razer Chroma App Config", NoClear, meta = (MaxLength = 256))
	FString AuthorName = TEXT("Enter Author Name");
	
	/**
	* Contact info for the author of this application (normally a support email or something)
	* 
	* Cannot be empty.
	* 
	* Char limit of 256.
	*/
	UPROPERTY(EditAnywhere, Category = "Razer Chroma App Config", NoClear, meta = (MaxLength = 256))
	FString AuthorContact = TEXT("Enter Contact");

	/**
	* A bitmask of the supported Razer Chroma devices for this project.
	*/
	UPROPERTY(EditAnywhere, Category = "Razer Chroma App Config", meta = (Bitmask, BitmaskEnum = "/Script/RazerChromaDevices.ERazerChromaDeviceTypes"))
	int32 SupportedDeviceTypes = static_cast<int32>(ERazerChromaDeviceTypes::All);

	/**
	* The category of this application in Razer Synapse. 
	* 
	* Default: 2
	*/
	UPROPERTY(VisibleAnywhere, Category = "Razer Chroma App Config")
	int32 Category = 2;
};

/**
 * Project settings for the Razer Chroma API in Unreal. 
 */
UCLASS(MinimalAPI, Config=Game, DefaultConfig, meta=(DisplayName="Razer Chroma Settings"))
class URazerChromaDevicesDeveloperSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	/**
	* If true then on module startup with should use the Razer App Info to populate
	* info about this application in Razer Synapse
	*/
	UE_API const bool ShouldUseChromaAppInfoForInit() const;

	/**
	* Returns the razer chroma app info
	* 
	* @see URazerChromaDevicesDeveloperSettings::AppInfo
	*/
	UE_API const FRazerChromaAppInfo& GetRazerAppInfo() const;

	/**
	* If true, Razer Chroma will be enabled.
	* 
	* @see URazerChromaDevicesDeveloperSettings::bIsRazerChromaEnabled
	*/
	UE_API const bool IsRazerChromaEnabled() const;

	/**
	* If true, then a IInputDevice will be created for Razer Chroma. 
	* 
	* @see URazerChromaDevicesDeveloperSettings::bCreateRazerChromaInputDevice
	*/
	UE_API const bool ShouldCreateRazerInputDevice() const;

	/**
	* This is the chroma animation file that should play when there are no other 
	* animations playing. 
	*/
	UE_API const URazerChromaAnimationAsset* GetIdleAnimation() const;
	
protected:

	//~Begin UDeveloperSettings interface
	UE_API virtual FName GetCategoryName() const override;
	//~End UDeveloperSettings interface

	/**
	 * If true, Razer Chroma will be enabled.
	 *
	 * Useful for if you need to "hot fix" Chroma off in case something goes wrong.
	 * If this is false, we won't even load the Razer Chroma.dll file at all or attempt to
	 * open any animation files associated with Razer Chroma.
	 */
	UPROPERTY(Config, EditAnywhere, Category="Razer Chroma", meta = (ConfigRestartRequired = true))
	bool bIsRazerChromaEnabled = true;

	/**
	* If true, then a IInputDevice will be created for Razer Chroma. 
	* 
	* This Input Device will allow for Razer Chroma to support Input Device Properties such as setting
	* the light color. This is desirable if you would like Razer Chroma to "just work" with any previous
	* implementations of setting a device's light color that you may have in your project.
	* 
	* If you set this to false, the Razer Chroma function library will still work, but any Input Device Properties
	* will not.
	* 
	* @see FRazerChromaInputDevice::SetDeviceProperty
	*/
	UPROPERTY(Config, EditAnywhere, Category="Razer Chroma", meta = (ConfigRestartRequired = true, EditCondition = "bIsRazerChromaEnabled"))
	bool bCreateRazerChromaInputDevice = false;

	/** 
	* This is the chroma animation file that should play when there are no other 
	* animations playing. 
	* 
	* If this is null then no idle animation will be set on boot
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Razer Chroma")
	TObjectPtr<URazerChromaAnimationAsset> IdleAnimationAsset;

	/**
	* If true, then the razer app will be initialized using the additional "App Data"
	* properties specified below. If this is false, the chroma SDK will be initialized 
	* without any additional information like what the name of the application is
	* or what device types it supports. 
	* 
	* You very likely will want to fill this out for your game.
	*/
	UPROPERTY(Config, EditAnywhere, Category="Razer Chroma", meta = (ConfigRestartRequired = true))
	bool bUseChromaAppInfoForInit = true;

	/**
	* Some definitions about your app that is used to initialize Razer Chroma.
	* 
	* This info will be used to populate Razer Synapse.
	*/
	UPROPERTY(Config, EditAnywhere, Category = "App Configuration", meta = (ConfigRestartRequired = true, EditCondition="bUseChromaAppInfoForInit"))
	FRazerChromaAppInfo AppInfo = {};
};

#undef UE_API
