// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "RazerChromaDevicesDeveloperSettings.h"	// For ERazerChromaDeviceTypes

#include "RazerChromaFunctionLibrary.generated.h"

#define UE_API RAZERCHROMADEVICES_API

class URazerChromaAnimationAsset;

/**
* Function library for Razer Chroma devices.
* 
* This function library is the main way that your gameplay code will likley interact with the 
* Razer Chroma API to play animations or set any custom effects
*/
UCLASS(MinimalAPI)
class URazerChromaFunctionLibrary final : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	* Returns true if the Razer Chroma runtime libraries are currently available.
	* 
	* This will be false on any machines that do not have Razer Chroma installed on them,
	* and thus cannot set any Razer Chroma effects.
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API bool IsChromaRuntimeAvailable();

	/**
	* Attempts to play the given Chroma animation file.
	* 
	* If the Chroma Runtime is not available, nothing will happen.
	* 
	* @param AnimToPlay		The Razer Chroma animation asset.
	* @param bLooping		If true, this animation will loop (start re-playing after it finishes)
	* 
	* @return	True if successfully played, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma", meta = (ReturnDisplayName = "Was Successful"))
	static UE_API bool PlayChromaAnimation(const URazerChromaAnimationAsset* AnimToPlay, const bool bLooping = false);

	/**
	* Returns true if the given animation is currently playing.
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, BlueprintCosmetic, Category = "Razer Chroma", meta = (ReturnDisplayName = "Is Playing"))
	static UE_API bool IsAnimationPlaying(const URazerChromaAnimationAsset* Anim);

	/**
	* Stops the given Chroma Animation if it is currently playing.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API void StopChromaAnimation(const URazerChromaAnimationAsset* AnimToStop);

	/**
	* Pauses the given animation if it is currently playing
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API void PauseChromaAnimation(const URazerChromaAnimationAsset* AnimToPause);

	/**
	 * Returns true if the given animation is currently paused.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API bool IsChromaAnimationPaused(const URazerChromaAnimationAsset* Anim);
	
	/**
	* Resumes the given animation if it has been paused.
	* 
	* @param AnimToResume	The animation to resume.
	* @param bLoop			If true, this animation will loop (start re-playing after it finishes)
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API void ResumeChromaAnimation(const URazerChromaAnimationAsset* AnimToResume, const bool bLoop);

	/**
	* Stops all currently active Chroma animations
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API void StopAllChromaAnimations();

	/**
	* Sets the idle animation for this application. This animation will play if no other animations are playing 
	* at the moment.
	* 
	* By default, the idle animation is set via the project settings, but it can be changed at runtime.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API void SetIdleAnimation(const URazerChromaAnimationAsset* NewIdleAnimation);

	/**
	* Sets whether or not we should use an idle currently an animation 
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API void SetUseIdleAnimation(const bool bUseIdleAnimation);

	/**
	 * Returns the duration in seconds of the specified animation.
	 */
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API float GetTotalDuration(const URazerChromaAnimationAsset* Anim);

	/**
	 * Sets the color of every connected Razer Chroma Device to this static color
	 *
	 * @param ColorToSet		The color to set the devices to
	 * @param DeviceTypes		Which types of razer devices you would like to set the color on if they are available
	 */
	static UE_API void SetAllDevicesStaticColor(const FColor& ColorToSet, const ERazerChromaDeviceTypes DeviceTypes = ERazerChromaDeviceTypes::All);
	
	/**
	* Sets the color of every connected Razer Chroma Device to this static color
	* 
	* @param ColorToSet		The color to set the devices to (the alpha channel is not used)
	* @param DeviceTypes	Which types of razer devices you would like to set the color on if they are available
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma", meta=(AutoCreateRefTerm="ColorToSet"))
	static UE_API void SetAllDevicesStaticColor(const FColor& ColorToSet, UPARAM(meta = (Bitmask, BitmaskEnum = "/Script/RazerChromaDevices.ERazerChromaDeviceTypes")) const int32 DeviceTypes);

	/**
	 * Converts ERazerChromaDeviceTypes enum to FString.
	 */
	static UE_API FString LexToString(const ERazerChromaDeviceTypes DeviceTypes);
	
	/** Converts a ERazerChromaDeviceTypes enum to string. */
	UFUNCTION(BlueprintPure, Category = "Utilities|String", meta = (DisplayName = "To String (ERazerChromaDeviceTypes)", CompactNodeTitle = "->", BlueprintAutocast))
	static UE_API FString Conv_RazerChromaDeviceTypesToString(UPARAM(meta = (Bitmask, BitmaskEnum = "/Script/RazerChromaDevices.ERazerChromaDeviceTypes")) const int32 DeviceTypes);

	/*
	* Name the Chroma event to add extras like haptics to supplement the event
	*
	* @param name			Empty string will stop haptic playback
	*						Name specifies an identifier that adds extras to game events like haptics
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API int32 SetEventName(const FString& name);

	/*
	* On by default, `UseForwardChromaEvents` sends the animation name to `SetEventName`
	* automatically when `PlayAnimationName` is called.
	*
	* @param bToggle		If true, PlayAnimation calls will pass the animation note to SetEventName.
	*						If false, PlayAnimation will not invoke SetEventName.
	*/
	UFUNCTION(BlueprintCallable, BlueprintCosmetic, Category = "Razer Chroma")
	static UE_API void UseForwardChromaEvents(const bool bToggle);
};

#undef UE_API
