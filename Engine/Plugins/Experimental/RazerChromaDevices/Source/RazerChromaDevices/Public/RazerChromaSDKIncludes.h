// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// These are the included files you need to use any native Razer Chroma SDK types
// to actually call into their API to set any lighting effects.

#if RAZER_CHROMA_SUPPORT

#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include "RazerChromaSDK/RzChromaSDKDefines.h"
#include "RazerChromaSDK/RzChromaSDKTypes.h"
#include "RazerChromaSDK/RzErrors.h"
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#endif	// #if RAZER_CHROMA_SUPPORT