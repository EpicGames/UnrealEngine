// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool.Rules
{
	public class RazerChromaDevices : ModuleRules
	{
		public RazerChromaDevices(ReadOnlyTargetRules Target) : base(Target)
		{
			// Enable truncation warnings in this plugin
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"ApplicationCore",
					"Engine",
					"Projects",
					"InputCore",
					"InputDevice",
					"CoreUObject",
					"DeveloperSettings",
				}
			);

			PrivateDependencyModuleNames.Add("RazerChromaSDK");


			// Determine if this platform can support Razer Chroma
			// TODO: This check may need to get more complicated.
			bool bPlatformSupportsChroma = Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && (Target.Architecture != UnrealArch.Arm64);
			
			// This module should _compile_ on all platforms, that way you can safely reference content 
			// from it and packaged to all platforms still. But it should only actually attempt to run 
			// on platforms that Razer Chroma is supported on.
			PublicDefinitions.Add("RAZER_CHROMA_SUPPORT=" + (bPlatformSupportsChroma ? "1" : "0"));			
		}
	}
}
