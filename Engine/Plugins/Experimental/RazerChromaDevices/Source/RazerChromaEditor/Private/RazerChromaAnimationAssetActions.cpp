// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaAnimationAssetActions.h"
#include "RazerChromaFunctionLibrary.h"
#include "RazerChromaAnimationAsset.h"
#include "AssetRegistry/AssetData.h"
#include "Framework/Commands/UIAction.h"

#include "ToolMenuDelegates.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "RazerAssetTypeActions"

FAssetTypeActions_RazerChromaPreviewAction::FAssetTypeActions_RazerChromaPreviewAction(const uint32 InCategoryBit)
		: CategoryBit(InCategoryBit)
{

}

UClass* FAssetTypeActions_RazerChromaPreviewAction::GetSupportedClass() const
{
	return URazerChromaAnimationAsset::StaticClass();
}

void FAssetTypeActions_RazerChromaPreviewAction::GetActions(const TArray<UObject*>& InObjects, FToolMenuSection& Section)
{
	TArray<TWeakObjectPtr<URazerChromaAnimationAsset>> Anims = GetTypedWeakObjectPtrs<URazerChromaAnimationAsset>(InObjects);

	Section.AddMenuEntry(
		"RazerChromaAnimPlayEffect",
		LOCTEXT("RazerChromaAnimPlayEffect", "Play"),
		LOCTEXT("RazerChromaAnimPlayEffectTooltip", "Plays the selected Razer Chroma Animation"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Play"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_RazerChromaPreviewAction::ExecutePlayAnim, Anims),
			FCanExecuteAction::CreateSP(this, &FAssetTypeActions_RazerChromaPreviewAction::CanExecutePlayCommand, Anims)
		)
	);

	Section.AddMenuEntry(
		"RazerChromaAnimStopEffect",
		LOCTEXT("RazerChromaAnimStopEffect", "Stop"),
		LOCTEXT("RazerChromaAnimStopEffectTooltip", "Stops the selected Razer Chroma Animation"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Toolbar.Stop"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_RazerChromaPreviewAction::ExecuteStopAnim, Anims),
			FCanExecuteAction()
		)
	);
}

bool FAssetTypeActions_RazerChromaPreviewAction::AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType)
{
	return false;
}

FText FAssetTypeActions_RazerChromaPreviewAction::GetName() const
{
	return LOCTEXT("AssetTypeActions_RazerChromaPreviewAction", "Razer Chroma Animation");
}

FColor FAssetTypeActions_RazerChromaPreviewAction::GetTypeColor() const
{
	// Kind of a "Razer Green"
	return FColor(0, 175, 0);
}

uint32  FAssetTypeActions_RazerChromaPreviewAction::GetCategories()
{
	return CategoryBit;
}

FText FAssetTypeActions_RazerChromaPreviewAction::GetAssetDescription(const FAssetData& AssetData) const
{
	return LOCTEXT("RazerChromaAnimAsset_Desc", "Represents a Razer Chroma animation asset, which can be imported from a '.chroma' file after being created in the Razer Chroma tools suite."); 
}

void FAssetTypeActions_RazerChromaPreviewAction::ExecutePlayAnim(TArray<TWeakObjectPtr<URazerChromaAnimationAsset>> Objects)
{
	for (const TWeakObjectPtr<URazerChromaAnimationAsset>& EffectPtr : Objects)
	{
		if (URazerChromaAnimationAsset* Effect = EffectPtr.Get())
		{
			URazerChromaFunctionLibrary::PlayChromaAnimation(Effect);
		}
	}
}

void FAssetTypeActions_RazerChromaPreviewAction::ExecuteStopAnim(TArray<TWeakObjectPtr<URazerChromaAnimationAsset>> Objects)
{
	for (const TWeakObjectPtr<URazerChromaAnimationAsset>& EffectPtr : Objects)
	{
		if (URazerChromaAnimationAsset* Effect = EffectPtr.Get())
		{
			// Stop all Razer Chroma animations that are playing
			URazerChromaFunctionLibrary::StopAllChromaAnimations();
			break;
		}
	}
}

bool FAssetTypeActions_RazerChromaPreviewAction::CanExecutePlayCommand(TArray<TWeakObjectPtr<URazerChromaAnimationAsset>> Objects) const
{
	// We can play a preview animation as long as there is one selected
	return !Objects.IsEmpty();
}

#undef LOCTEXT_NAMESPACE