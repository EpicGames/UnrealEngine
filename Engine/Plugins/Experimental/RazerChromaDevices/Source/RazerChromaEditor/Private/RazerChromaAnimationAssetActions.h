// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetTypeActions_Base.h"

class URazerChromaAnimationAsset;

/**
* Asset Actions for the URazerChromaAnimationAsset which allow you to preview 
* the animation in the editor without having to PIE. 
*/
class FAssetTypeActions_RazerChromaPreviewAction : public FAssetTypeActions_Base
{
public:
	FAssetTypeActions_RazerChromaPreviewAction(const uint32 InCategoryBit);

	virtual FText GetName() const override;
	virtual FColor GetTypeColor() const override;
	virtual uint32 GetCategories() override;
	virtual UClass* GetSupportedClass() const override;
	virtual void GetActions(const TArray<UObject*>& InObjects, struct FToolMenuSection& Section) override;
	virtual bool AssetsActivatedOverride(const TArray<UObject*>& InObjects, EAssetTypeActivationMethod::Type ActivationType) override;
	virtual FText GetAssetDescription(const FAssetData& AssetData) const override;

protected:

	void ExecutePlayAnim(TArray<TWeakObjectPtr<URazerChromaAnimationAsset>> Objects);
	void ExecuteStopAnim(TArray<TWeakObjectPtr<URazerChromaAnimationAsset>> Objects);
	bool CanExecutePlayCommand(TArray<TWeakObjectPtr<URazerChromaAnimationAsset>> Objects) const;

	uint32 CategoryBit;
};