// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RazerChromaDeviceLogging.h"
#include "AssetToolsModule.h"
#include "RazerChromaAnimationAssetActions.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"

/** Custom style set for Razer Chroma */
class FRazerChromaDevicesSlateStyle final : public FSlateStyleSet
{
public:
	FRazerChromaDevicesSlateStyle()
		: FSlateStyleSet("RazerChromaDevicesEditor")
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		const FString PluginDirectory = IPluginManager::Get().FindPlugin(TEXT("RazerChromaDevices"))->GetBaseDir();
		SetContentRoot(PluginDirectory / TEXT("Content/Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// Icon sizes
		const FVector2D Icon16 = FVector2D(16.0, 16.0);
		const FVector2D Icon64 = FVector2D(64.0, 64.0);
		
		Set("ClassIcon.RazerChromaAnimationAsset", new IMAGE_BRUSH_SVG("Icons/ChromaAnimation_16", Icon16));
		Set("ClassThumbnail.RazerChromaAnimationAsset", new IMAGE_BRUSH_SVG("Icons/ChromaAnimation_64", Icon64));
	}
};

class FRazerChromaEditorModule : public IModuleInterface
{
protected:

	TSharedPtr<FAssetTypeActions_RazerChromaPreviewAction> RazerChromaPreviewAction;

	TSharedPtr<FSlateStyleSet> StyleSet;

	virtual void StartupModule() override
	{
		// Register our custom asset actions for razer chroma animation assets
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools")).Get();
		EAssetTypeCategories::Type Category = AssetTools.RegisterAdvancedAssetCategory(FName(TEXT("Razer")), NSLOCTEXT("RazerChromaAnimations", "RazerChromaAnimMenu", "Razer Chroma"));
		RazerChromaPreviewAction = MakeShareable(new FAssetTypeActions_RazerChromaPreviewAction(Category));
		AssetTools.RegisterAssetTypeActions(RazerChromaPreviewAction.ToSharedRef());

		// Make a new style set for Razer Chroma
		StyleSet = MakeShared<FRazerChromaDevicesSlateStyle>();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
	}
	
	virtual void ShutdownModule() override
	{
		if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetTools")))
		{
			IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
			AssetTools.UnregisterAssetTypeActions(RazerChromaPreviewAction.ToSharedRef());
		}

		// Unregister slate stylings
		if (StyleSet.IsValid())
		{
			FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		}
	}
};

IMPLEMENT_MODULE(FRazerChromaEditorModule, RazerChromaEditor);