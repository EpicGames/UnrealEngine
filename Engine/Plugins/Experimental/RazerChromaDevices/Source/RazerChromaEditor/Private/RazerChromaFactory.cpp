// Copyright Epic Games, Inc. All Rights Reserved.

#include "RazerChromaFactory.h"
#include "FileMediaSource.h"
#include "Misc/Paths.h"
#include "Misc/MessageDialog.h"
#include "UObject/UObjectGlobals.h"
#include "Subsystems/ImportSubsystem.h"
#include "Editor/EditorEngine.h"
#include "Editor.h" // for GEditor
#include "RazerChromaAnimationAsset.h"
#include "CoreMinimal.h"
#include "Misc/FeedbackContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RazerChromaFactory)

URazerChromaFactory::URazerChromaFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("chroma;Razer Chroma Animation"));
	bEditorImport = true;

	SupportedClass = URazerChromaAnimationAsset::StaticClass();
}

bool URazerChromaFactory::FactoryCanImport(const FString& Filename)
{
	const FString Extension = FPaths::GetExtension(*Filename);
	if (Extension == TEXT("chroma"))
	{
		return true;
	}
	
	return Super::FactoryCanImport(Filename);
}

UObject* URazerChromaFactory::FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn)
{
	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, Type);
	
	// Refuse to accept big files. We currently use TArray<> which will fail if we go over an int32.
	{
		const uint64 Size = BufferEnd - Buffer;
		if (!IntFitsIn<int32>(Size))
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("File '%s' is too big (%umb), Max=%umb"), *InName.ToString(), Size >> 20, TNumericLimits<int32>::Max() >> 20);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}

		// This is just an assumption to try and validate binary files here... any animation file should certainly be larger then 4 bytes
		static constexpr uint64 MinFileSize = 4;

		if (Size <= MinFileSize)
		{
			Warn->Logf(ELogVerbosity::Error, TEXT("File '%s' is too small (%u), Min=%u"), *InName.ToString(), Size >> 20, MinFileSize);
			GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
			return nullptr;
		}
	}

	// Create a new razer animation asset and copy over the byte data for this razer chroma file
	URazerChromaAnimationAsset* ChromaObject = NewObject<URazerChromaAnimationAsset>(InParent, InName, Flags);
	const bool bSuccessfulImport = ChromaObject->ImportFromFile(GetCurrentFilename(), Buffer, BufferEnd);
	
	if (!bSuccessfulImport)
	{
		// Inform user we failed to create the sound wave
		Warn->Logf(ELogVerbosity::Error, TEXT("Failed to import Razer Chroma Animation %s"), *InName.ToString());
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ChromaObject);
	
	return ChromaObject;
}
