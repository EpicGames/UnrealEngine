// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"

#include "RazerChromaFactory.generated.h"

UCLASS(hidecategories=Object)
class URazerChromaFactory
	: public UFactory
{
	GENERATED_BODY()

public:
	URazerChromaFactory(const FObjectInitializer& ObjectInitializer);
	
protected:

	//~ Begin UFactory Interface
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateBinary(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn) override;
	//~ End UFactory Interface
};
