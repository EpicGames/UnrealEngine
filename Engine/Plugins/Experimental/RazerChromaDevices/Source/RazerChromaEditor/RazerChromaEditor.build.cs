// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RazerChromaEditor : ModuleRules
	{
		public RazerChromaEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"MediaAssets",
					"UnrealEd",
					"Engine",
					"RazerChromaDevices",
					"SlateCore",
					"Slate",
					"ToolMenus",
					"Projects"
				});
			
			PrivateDependencyModuleNames.Add("RazerChromaSDK");
			
		}
	}
}