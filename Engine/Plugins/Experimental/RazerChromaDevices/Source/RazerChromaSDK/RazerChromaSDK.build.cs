// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class RazerChromaSDK : ModuleRules
	{
		public RazerChromaSDK(ReadOnlyTargetRules Target) : base(Target)
		{
			Type = ModuleType.External;
			
			// TODO: We may want to suport other platforms in the future as well
			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
			{
				// Add the third party include folders so that we can use Razer types
				PublicSystemIncludePaths.Add(Path.Combine(ModuleDirectory, "ThirdParty"));
				
				if (Target.Architecture != UnrealArch.Arm64)
				{
					// Redist the CChromaEditorLibrary64.dll with the game
					string DllPath = Path.GetFullPath(Path.Combine(PluginDirectory, "Binaries/ThirdParty/Win64"));

					RuntimeDependencies.Add(Path.Combine(DllPath, "CChromaEditorLibrary64.dll"));

					// We only want the debug symbols outside of shipping
					if (Target.Configuration != UnrealTargetConfiguration.Shipping)
					{
						// The PDB file may not exist if cloning from GitHub. See the README for
						// where to put it if you need extra debug symbols.
						string PDBPath = Path.Combine(DllPath, "CChromaEditorLibrary64.pdb");
						if (File.Exists(PDBPath))
						{
							RuntimeDependencies.Add(PDBPath);
						}						
					}					
				}
			}
		}
	}
}
